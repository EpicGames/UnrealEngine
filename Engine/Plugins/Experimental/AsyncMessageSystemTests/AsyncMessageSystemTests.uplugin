{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"Description": "Async Message System Tests",
	"Category": "Framework",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AsyncMessageSystemTests",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "AsyncMessageSystem",
			"Enabled": true
		},
		{
			"Name": "RuntimeTests",
			"Enabled": true
		}
	]
}