// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AsyncMessageSystemTests : ModuleRules
	{
		public AsyncMessageSystemTests(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			// Enable truncation warnings in this plugin
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			
			// Uncomment this line to make for easier debugging
			//OptimizeCode = CodeOptimization.Never;
			
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AsyncMessageSystem",
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"RuntimeTests",
				}
			);
		}
	}
}