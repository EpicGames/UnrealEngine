// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageSystemPerformanceTests.h"
#include "AsyncMessageHandle.h"
#include "AsyncMessageSystemBase.h"
#include "Misc/AutomationTest.h"
#include "NativeGameplayTags.h"
#include "EngineRuntimeTests.h"
#include "AsyncGameplayMessageSystem.h"
#include "AsyncMessageWorldSubsystem.h"
#include "Stats/StatsMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageSystemPerformanceTests)

static TAutoConsoleVariable<int32> CVarAsyncMessageSystemTestActorCount(
	TEXT("AsyncMessageSystem.Tests.Performance.ActorCount"),
	4000,
	TEXT("Number of actors to spawn for tick test\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarAsyncMessageSystemTestTickCount(
	TEXT("AsyncMessageSystem.Tests.Performance.TickCount"),
	2000,
	TEXT("Number of frames to tick\n"),
	ECVF_Default);

void AASyncMessagePerfTest::SetupBindingToMessage(
	const FAsyncMessageId& MessageToBindTo,
	const FAsyncMessageBindingOptions& BindingOpts)
{
	auto MessageSys = UAsyncMessageWorldSubsystem::GetSharedMessageSystem(GetWorld());
	check(MessageSys.IsValid());

	// Bind to the message that was passed in
	const FAsyncMessageHandle Handle = MessageSys->BindListener(MessageToBindTo, TWeakObjectPtr<AASyncMessagePerfTest> { this }, &AASyncMessagePerfTest::HandleTestCallback, BindingOpts);
	ensure(Handle.IsValid());
	BoundHandles.Emplace(Handle);
}

void AASyncMessagePerfTest::HandleTestCallback(const FAsyncMessage& Message)
{
	if (const FAsyncMessagePerfTestPayload* Data = Message.GetPayloadData<const FAsyncMessagePerfTestPayload>())
	{
		if (Data->bDoLessWork)
		{
			DoSimpleTestWork();
		}
		else
		{
			// When we receive the callback, then run our virtual tick function
			DoTestWork();
		}
	}
}

void AASyncMessagePerfTest::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	// Unbind all the listeners for this actor
	auto Sys = UAsyncMessageWorldSubsystem::GetSharedMessageSystem(GetWorld());
	if (Sys.IsValid())
	{
		for (const FAsyncMessageHandle& BoundHandle : BoundHandles)
		{
			Sys->UnbindListener(BoundHandle);
		}
	}
}

void AASyncMessagePerfTest::DoTestWork()
{
	// Some simple work to simulate doing something
	if (MathIncrement > 0.0f && MathLimit > 0.0f)
	{
		MathCounter = 0.0f;
		while (MathCounter < MathLimit)
		{
			MathCounter += MathIncrement;
		}
	}
}

void AASyncMessagePerfTest::DoSimpleTestWork()
{
	float SimpleMathIncrement = 0.10f;
	float SimpleMathLimit = 1.0f;
	
	// Some simple work to simulate doing something
	if (SimpleMathIncrement > 0.0f && SimpleMathLimit > 0.0f)
	{
		float SimpleMathCounter = 0.0f;
		while (SimpleMathCounter < SimpleMathLimit)
		{
			SimpleMathCounter += SimpleMathIncrement;
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS

constexpr EAutomationTestFlags PerformanceTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::EngineFilter;

namespace UE::AsyncMessageSystem
{
	UE_DEFINE_GAMEPLAY_TAG_COMMENT(TestMessage_RunVirtualTick, "AsyncMessages.Internal.test.RunVirtualTick", "A test gameplay tag utilized in the async message system unit tests to call virtual tick functions on test actors")
	const FAsyncMessageId RunVirtualTickMessageId = { TestMessage_RunVirtualTick };
	
	UE_DEFINE_GAMEPLAY_TAG_COMMENT(TestMessage_DoSomeFakeWork, "AsyncMessages.Internal.test.DoSomeFakeWork", "A test gameplay tag utilized in the async message system unit tests to call virtual tick functions on test actors")
	const FAsyncMessageId DoSomeFakeWorkMessageId = { TestMessage_DoSomeFakeWork };

	/**
	 * An automation test capable of creating test worlds and ticking them for the async message system
	 * This test will automatically create the number of actors specified by "CVarAsyncMessageSystemTestActorCount"
	 * and tick them "CVarAsyncMessageSystemTestTickCount" times.
	 *
	 * When spawned, the actors will bind the RunVirtualTickMessageId message ID and it's parent tag, and be
	 * evenly distributed across 6 different tick groups.
	 */
	class FAsyncMessageSystemTestBase : public FEngineTickTestBase
	{
	public:
		FAsyncMessageSystemTestBase(const FString& InName, const bool bInComplexTask)
			: FEngineTickTestBase(InName, bInComplexTask)
		{
			
		}

		// The number of actors spawned in RunAsyncMessageTestSetup
		int32 ActorCount = -1;

		// The number of times this test is currently ticking
		int32 TickCount = -1;

		float SimulatedDeltaTime = 0.01f;

		/**
		 * Bitmask of test flags for the async message system
		 */
		enum class EAsyncMessagePerfTestFlags : uint8 
		{
			Empty = 0x00,
			SetupTickPrerequisiteOnTestActors = 0x01,
			SetupTickIntervalOnTestActors = 0x02,
			All = SetupTickPrerequisiteOnTestActors | SetupTickIntervalOnTestActors
		};
		
		// Initalizes the test world and creates the specifed number of actors and their tick dependencies
		bool RunAsyncMessageTestSetup(const EAsyncMessagePerfTestFlags Flags = EAsyncMessagePerfTestFlags::Empty, const int32 ActorCountOverride = -1, const float InSimulatedDeltaTime = 0.01f)
		{
			bool bSuccess = true;
			SimulatedDeltaTime = InSimulatedDeltaTime;

			// Init the world
			if (!CreateTestWorld())
			{
				ReportAnyErrors();
				return false;
			}

			// Destroy any test actors which may already exist, just in case we want to run this multiple times in one test
			DestroyAllTestActors();

			// Store how many actors we are using
			ActorCount = ActorCountOverride > 0 ? ActorCountOverride : CVarAsyncMessageSystemTestActorCount.GetValueOnAnyThread();

			// Create the test actors
			bSuccess &= CreateTestActors(ActorCount, AASyncMessagePerfTest::StaticClass());

			if (Flags != EAsyncMessagePerfTestFlags::Empty)
			{
				// Psuedo-random actor tick pre-reqs copied from EngineRuntimeTests
				const int32 RandomSeed = 0xABCD1234;
				FRandomStream RandomSource(RandomSeed);
				
				// Setup tick pre-requisetes
				for (int32 i = 0; i < ActorCount; i++)
				{
					if (EnumHasAnyFlags(Flags, EAsyncMessagePerfTestFlags::SetupTickPrerequisiteOnTestActors) && i != (ActorCount-1))
					{
						// Enable dependencies on a random later actor
						TestActors[i]->AddTickPrerequisiteActor(TestActors[RandomSource.RandRange(i+1, ActorCount-1)]);
					}
					
					if (EnumHasAnyFlags(Flags, EAsyncMessagePerfTestFlags::SetupTickIntervalOnTestActors))
					{
						// Enable a small interval, this should not affect actual timing
						TestActors[i]->SetActorTickInterval(SimulatedDeltaTime / 2.f + (float)RandomSource.FRandRange(-SimulatedDeltaTime/10.f, SimulatedDeltaTime/10.f));
					}
				}
				
			}

			// Lastly, begin play in the test world
			bSuccess &= BeginPlayInTestWorld();

			// and tick the world once without any tests running
			// Tick the world once before the test ticks to give any underlying systems a chance to finish initaliziating, like the task sync manager.
			if (true)
			{
				TickTestWorld(SimulatedDeltaTime);
				TickCount++;
			}

			return bSuccess;
		}

		void AddBindingsToAllTestActors(const TArray<FAsyncMessageId>& MessagesToListenFor, const bool bOverrideBindOptions = false, const TArray<FAsyncMessageBindingOptions>& OverrideBindOptions = {})
		{
			check(ActorCount >= 0);
			
			// A naive helper to evenly distribute the binding of message across 6 different tick groups 
			auto GetTickGroupForIdx = [CurrentActorCount = ActorCount](const int32 Index) -> ETickingGroup
			{
				const float fActorCount = (float)CurrentActorCount;
				const float fIdx = (float)Index;

				const float Perc = 100.0f * (float)fIdx / (float)fActorCount;

				if (Perc <= 16.66f)
				{
					return TG_PrePhysics;
				}
				else if (Perc <= 33.33f)
				{
					return TG_StartPhysics;
				}
				else if (Perc <= 50.0f)
				{
					return TG_DuringPhysics;
				}
				else if (Perc <= 66.66f)
				{
					return TG_EndPhysics;
				}
				else if (Perc <= 83.33f)
				{
					return TG_PostPhysics;
				}
				else
				{
					return TG_PostUpdateWork;
				}
			};

			// Bind all the test actors to virtual tick
			for (int32 i = 0; i < ActorCount; ++i)
			{
				AASyncMessagePerfTest* PerfTestActor = CastChecked<AASyncMessagePerfTest>(TestActors[i]);
				
				if  (!PerfTestActor)
				{
					continue;
				}
				
				// If you override the binding options, then bind to all of them which are given
				if (bOverrideBindOptions)
				{
					for (const FAsyncMessageBindingOptions& OverridenBindingOption : OverrideBindOptions)
					{
						for (const FAsyncMessageId& MessageToBind : MessagesToListenFor)
						{
							PerfTestActor->SetupBindingToMessage(MessageToBind, OverridenBindingOption);	
						}
					}
				}
				// Otherwise, evenly distributing the bindings among different listeners across different tick groups (this would be game thread only, because they are tick groups)
				else
				{
					FAsyncMessageBindingOptions BindingOptions;
					BindingOptions.SetTickGroup(GetTickGroupForIdx(i));
					
					for (const FAsyncMessageId& MessageToBind : MessagesToListenFor)
					{
						PerfTestActor->SetupBindingToMessage(MessageToBind, BindingOptions);	
					}
				}
			}
		}

		void ForEachTestActor(TFunction<void(AASyncMessagePerfTest*)> Callback)
		{
			for (int32 i = 0; i < ActorCount; ++i)
			{
				if  (AASyncMessagePerfTest* PerfTestActor = CastChecked<AASyncMessagePerfTest>(TestActors[i]))
				{
					Callback(PerfTestActor);
				}
			}
		}

		AEngineTestTickActor* GetPseudoRandomTestActor() const
		{
			constexpr int32 RandomSeed = 0xDCBA4321;
			FRandomStream RandomSource(RandomSeed);
			
			const int32 RandomIndex = RandomSource.RandRange(0, ActorCount - 1);

			ensure(TestActors.IsValidIndex(RandomIndex));
			return TestActors[RandomIndex];
		}
		
		// Runs the given TickLambda for the number of test ticks this system has
		void RunTestTicks(const TCHAR* TickTestName, TFunction<void(TSharedPtr<FAsyncGameplayMessageSystem>, float, int32)> TickLambda, const int32 TickCountOverride = -1)
		{
			TickCount = TickCountOverride > 0 ? TickCountOverride : CVarAsyncMessageSystemTestTickCount.GetValueOnAnyThread();
			
			// Ensure that the test actors are reset
			ResetTestActors();

			// Actually run our ticks of the test world
			{
				// This will give us a scope in UnrealInsights for the duration of ticking, which is the perf that we care about
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(TickTestName)
				// And this will nicely log out how long this scope takes in MS
				FScopeLogTime LogTimePtr(TickTestName, nullptr, FScopeLogTime::ScopeLog_Milliseconds);
				
				for (int32 i = 0; i < TickCount; i++)
				{
					TickLambda(GetGameplayMessageSystem(), SimulatedDeltaTime, i);
				
					// Tick normally to compare against other solution
					TickTestWorld(SimulatedDeltaTime);
				}
			}
			
			// Tests that each actor was ticked the correct number of times
			CheckTickCount(TickTestName, TickCount);
		}

		/**
		 * @return The gameplay message system associated with the test world.
		 */
		TSharedPtr<FAsyncGameplayMessageSystem> GetGameplayMessageSystem() const
		{
			return UAsyncMessageWorldSubsystem::GetSharedMessageSystem<FAsyncGameplayMessageSystem>(GetTestWorld());
		}
	};
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(TMessageSystemPerformance_BroadcastingMessages, UE::AsyncMessageSystem::FAsyncMessageSystemTestBase, "AsyncMessagePassing.Performance.GameThread.BroadcastSingleMessage", PerformanceTestFlags)
bool TMessageSystemPerformance_BroadcastingMessages::RunTest(const FString& Parameters)
{
	using namespace UE::AsyncMessageSystem;

	bool bSuccess = true;
	
	bSuccess &= RunAsyncMessageTestSetup();

	// Bind all test actors to one test message and its parent
	AddBindingsToAllTestActors({ RunVirtualTickMessageId, RunVirtualTickMessageId.GetParentMessageId() });

	// A simple tick which will queue one message for broadcasting each frame.
	auto TickLambda = [](TSharedPtr<FAsyncGameplayMessageSystem> MessageSys, float DeltaTime, int32 TickNum) -> void
	{
		FAsyncMessagePerfTestPayload PayloadData;
		FConstStructView PayloadView = FConstStructView::Make<FAsyncMessagePerfTestPayload>(PayloadData);

		MessageSys->QueueMessageForBroadcast(RunVirtualTickMessageId, PayloadView);
	};
	
	RunTestTicks(TEXT("AsyncMessage_BroadcastSingleMessage"), TickLambda);

	// Always reset test world
	bSuccess &= DestroyTestWorld();
	
	return bSuccess && !ReportAnyErrors();
}

// A test which will broadcast a lot of different message to listeners
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(TMessageSystemPerformance_BroadcastSeveralMessages, UE::AsyncMessageSystem::FAsyncMessageSystemTestBase, "AsyncMessagePassing.Performance.GameThread.BroadcastSeveralMessages", PerformanceTestFlags)
bool TMessageSystemPerformance_BroadcastSeveralMessages::RunTest(const FString& Parameters)
{
	using namespace UE::AsyncMessageSystem;

	bool bSuccess = true;
	
	bSuccess &= RunAsyncMessageTestSetup();

	const TArray<FAsyncMessageId> MessagesToBindAndBroadcast =
		{
			RunVirtualTickMessageId,
			RunVirtualTickMessageId.GetParentMessageId(),
			DoSomeFakeWorkMessageId
		};
	
	// Bind all test actors to one test message and its parent
	AddBindingsToAllTestActors(MessagesToBindAndBroadcast);

	FAsyncMessagePerfTestPayload PayloadData = { .TargetActor = nullptr, .bDoLessWork = true };
	FConstStructView PayloadView = FConstStructView::Make<FAsyncMessagePerfTestPayload>(PayloadData);
	
	auto TickLambda = [&MessagesToBindAndBroadcast, &PayloadView](TSharedPtr<FAsyncGameplayMessageSystem> MessageSys, float DeltaTime, int32 TickNum) -> void
	{
		for (const FAsyncMessageId& MessageId : MessagesToBindAndBroadcast)
		{
			// Note: We specifically do not want the cost of constructing the FInstancedStruct outside of the
			// message system in the profile. Here we can keep the cost related only to what the message system does
			MessageSys->QueueMessageForBroadcast(MessageId, PayloadView);	
		}
	};
	
	RunTestTicks(TEXT("AsyncMessage_BroadcastSeveralMessages"), TickLambda);

	// Always reset test world
	bSuccess &= DestroyTestWorld();
	
	return bSuccess && !ReportAnyErrors();
}

// Have listeners on a different thread and queue messages from different threads
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(TMessageSystemPerformance_BroadcastSeveralMessagesMultithread, UE::AsyncMessageSystem::FAsyncMessageSystemTestBase, "AsyncMessagePassing.Performance.MultiThread.BroadcastSeveralMessages", PerformanceTestFlags)
bool TMessageSystemPerformance_BroadcastSeveralMessagesMultithread::RunTest(const FString& Parameters)
{
	using namespace UE::AsyncMessageSystem;

	bool bSuccess = true;

	// Set up ticking pre-reqs on actors to make for a more realistic gameplay scenario
	bSuccess &= RunAsyncMessageTestSetup(EAsyncMessagePerfTestFlags::All);

	// Add some bindings on the main game thread to all listeners
	const TArray<FAsyncMessageId> GameThreadMessages =
	{
		RunVirtualTickMessageId,
		RunVirtualTickMessageId.GetParentMessageId() 
	};
	
	// Add some test work to be bound on the game thread
	AddBindingsToAllTestActors(GameThreadMessages);
	
	// Also add several different bindings for each actor on various background threads.
	// This will make the system need to keep track of more message queues (one for each binding option)
	// and copy the payload data more, allowing us to really stress test and see where there is possible contention.
	const TArray<FAsyncMessageBindingOptions> BindingsToUse = 
	{
		FAsyncMessageBindingOptions(ENamedThreads::HighTaskPriority),
		FAsyncMessageBindingOptions(ENamedThreads::GameThread),
		FAsyncMessageBindingOptions(ENamedThreads::RHIThread),
		FAsyncMessageBindingOptions(UE::Tasks::ETaskPriority::Default, UE::Tasks::EExtendedTaskPriority::Inline),
		FAsyncMessageBindingOptions(UE::Tasks::ETaskPriority::ForegroundCount, UE::Tasks::EExtendedTaskPriority::TaskEvent),
		FAsyncMessageBindingOptions(UE::Tasks::ETaskPriority::BackgroundNormal, UE::Tasks::EExtendedTaskPriority::GameThreadHiPri),
	};
	
	ForEachTestActor([&BindingsToUse](AASyncMessagePerfTest* TestActor)
	{
		if (TSharedPtr<FAsyncGameplayMessageSystem> Sys = UAsyncMessageWorldSubsystem::GetSharedMessageSystem<FAsyncGameplayMessageSystem>(TestActor->GetWorld()))
		{
			for (const FAsyncMessageBindingOptions& BindOptions : BindingsToUse)
			{
				// Add a lamda function which will do some simple floating point test work in response to "DoSomeFakeWorkMessageId"
				const FAsyncMessageHandle Handle = Sys->BindListener(DoSomeFakeWorkMessageId, [WeakActor = TWeakObjectPtr<AASyncMessagePerfTest>(TestActor)](const FAsyncMessage& Message)
				{
					if (WeakActor.IsValid())
					{
						WeakActor->DoSimpleTestWork();	
					}
				}, BindOptions);

				TestActor->BoundHandles.Add(Handle);
			}
		}
	});

	TArray<FAsyncMessageId> MessagesToBindAndBroadcast;
	MessagesToBindAndBroadcast.Append(GameThreadMessages);

	TArray<UE::Tasks::FTask> PendingTasks;
	
	struct FQueueMessageFromBackgroundThread
	{
		TSharedPtr<FAsyncGameplayMessageSystem> MessageSystem;
		FAsyncMessageId MessageToQueue;
		FConstStructView PayloadDataView;
		void operator()()
		{
			if (MessageSystem.IsValid() && MessageToQueue.IsValid())
			{
				MessageSystem->QueueMessageForBroadcast(MessageToQueue, PayloadDataView);
			}
		}
	};

	FAsyncMessagePerfTestPayload PayloadData = { .TargetActor = nullptr, .bDoLessWork = true };
	FConstStructView PayloadView = FConstStructView::Make<FAsyncMessagePerfTestPayload>(PayloadData);

	constexpr int32 RandomSeed = 0xDCBA4321;
	FRandomStream TaskPriRandomSource(RandomSeed);
	
	auto GetRandomTaskPri = [&TaskPriRandomSource]() -> TTuple<UE::Tasks::ETaskPriority, UE::Tasks::EExtendedTaskPriority>
	{
		static const TArray<TTuple<UE::Tasks::ETaskPriority, UE::Tasks::EExtendedTaskPriority>> PrioritiesToChoseFrom =
		{
			{UE::Tasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::None},
			{UE::Tasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::TaskEvent},
			{UE::Tasks::ETaskPriority::BackgroundNormal, UE::Tasks::EExtendedTaskPriority::None},
			{UE::Tasks::ETaskPriority::High, UE::Tasks::EExtendedTaskPriority::None},
			{UE::Tasks::ETaskPriority::BackgroundHigh, UE::Tasks::EExtendedTaskPriority::None},
		};
		
		const int32 RandomIndex = TaskPriRandomSource.RandRange(0, PrioritiesToChoseFrom.Num() - 1);
		ensure(PrioritiesToChoseFrom.IsValidIndex(RandomIndex));
		
		return PrioritiesToChoseFrom[RandomIndex];
	};
	
	auto TickLambda = [&](TSharedPtr<FAsyncGameplayMessageSystem> MessageSys, float DeltaTime, int32 TickNum) -> void
	{
		// Queue messages from the game thread
		for (const FAsyncMessageId& MessageId : GameThreadMessages)
		{
			// Note: We specifically do not want the cost of constructing the FInstancedStruct outside of the
			// message system in the profile. Here we can keep the cost related only to what the message system does
			MessageSys->QueueMessageForBroadcast(MessageId, PayloadView);
		}

		// Pick a semi-random task priority to queue messages from a different thread each tick
		// Queue a message from a background thread
		const TTuple<UE::Tasks::ETaskPriority, UE::Tasks::EExtendedTaskPriority>& TaskPris = GetRandomTaskPri();

		FAsyncMessagePerfTestPayload AsyncPayloadData = { .TargetActor = nullptr, .bDoLessWork = true };
		FConstStructView AsyncPayloadView = FConstStructView::Make<FAsyncMessagePerfTestPayload>(AsyncPayloadData);
		
		UE::Tasks::FTask T =
			UE::Tasks::Launch(
				UE_SOURCE_LOCATION,
				FQueueMessageFromBackgroundThread { .MessageSystem = MessageSys, .MessageToQueue = DoSomeFakeWorkMessageId, .PayloadDataView = AsyncPayloadView },
				TaskPris.Key,
				TaskPris.Value);
		
		PendingTasks.Push(T);
	};
	
	RunTestTicks(TEXT("AsyncMessage_BroadcastSeveralMessages_Multi"), TickLambda);

	UE::Tasks::Wait(PendingTasks);

	// Always reset test world
	bSuccess &= DestroyTestWorld();
	
	return bSuccess && !ReportAnyErrors();
}

UE_DEFINE_GAMEPLAY_TAG_COMMENT(InternalTestTag_RefCollection, "AsyncMessages.Internal.test.ReferenceCollection", "A test gameplay tag utilized in the async message system unit tests to test reference collection")
static const FAsyncMessageId MessageId_RefCollection = { InternalTestTag_RefCollection };

// Test that the message system correctly keeps tracks of reference UPROPERTY's in it's payloads
// so that they do not get garbage collected whilst still on the message queue
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(TMessageSystem_PayloadReferenceTest, UE::AsyncMessageSystem::FAsyncMessageSystemTestBase, "AsyncMessagePassing.ReferenceCollection", PerformanceTestFlags)
bool TMessageSystem_PayloadReferenceTest::RunTest(const FString& Parameters)
{
	using namespace UE::AsyncMessageSystem;

	bool bSuccess = true;

	bSuccess &= RunAsyncMessageTestSetup(EAsyncMessagePerfTestFlags::All, /*actor count override*/15);

	// Bind a message to our event for populating a test actor on the game thread
	ForEachTestActor([&](AASyncMessagePerfTest* TestActor)
	{
		if (TSharedPtr<FAsyncGameplayMessageSystem> Sys = UAsyncMessageWorldSubsystem::GetSharedMessageSystem<FAsyncGameplayMessageSystem>(TestActor->GetWorld()))
		{
			FAsyncMessageBindingOptions BindOptions = {};
			
			// Add a lamda function which will do some simple floating point test work in response to "DoSomeFakeWorkMessageId"
			const FAsyncMessageHandle Handle = Sys->BindListener(MessageId_RefCollection, [&](const FAsyncMessage& Message)
			{						
				if (const FTest_RefCollection_Payload* Data = Message.GetPayloadData<const FTest_RefCollection_Payload>())
				{
					// Test to make sure that the payload object is valid and has not been GC'd
					TestTrue(TEXT("Test data object is a valid pointer"), Data->ObjPoint != nullptr);
					TestTrue(TEXT("Test data object is a Valid LowLevel"), Data->ObjPoint->IsValidLowLevel());
					TestFalse(TEXT("Test data object is a reachable object!"), Data->ObjPoint->IsUnreachable());

					if (Data->ObjPoint)
					{
						UE_LOGF(LogTemp, Log, "Test actor is valid!");

						if (Data->ObjPoint->IsUnreachable())
						{
							UE_LOGF(LogTemp, Error, "Test actor is unreachable! ruh roh");
						}
						// We crash here prior to any work
						else if (UTestRefCollectionObject* AsyncObj = Cast<UTestRefCollectionObject>(Data->ObjPoint))
						{
							UE_LOGF(LogTemp, Log, "Even the type info is there!");
						}							
					}
					else
					{
						UE_LOGF(LogTemp, Warning, "We have no test actor pointer");
					}
				}
			}, BindOptions);

			TestActor->BoundHandles.Add(Handle);
		}
	});

	// A simple tick which will queue one message for broadcasting each frame.
	auto TickLambda = [](TSharedPtr<::FAsyncGameplayMessageSystem> MessageSys, float DeltaTime, int32 TickNum) -> void
	{
		// On tick 1, create a UObject which would only be referenced by the message we queue to the system 
		if (TickNum == 1)
		{
			{
				UTestRefCollectionObject* CreatedObject = NewObject<UTestRefCollectionObject>(GetTransientPackage(), TEXT("TestObject_0"));

				FTest_RefCollection_Payload RefCollectionPayload {};
				RefCollectionPayload.ObjPoint = CreatedObject;
				FConstStructView PayloadView = FConstStructView::Make<FTest_RefCollection_Payload>(RefCollectionPayload);

				MessageSys->QueueMessageForBroadcast(MessageId_RefCollection, PayloadView);	
			}
			
			// And then immediately run GC. The Object will be collected because it won't be tracked anywhere
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			// The message system will then proceed to tick again, and we will see what happens in the lambda above. 
			// What will happen by default here is that the payload data object will not be referenced by anything, so
			// will be destroyed by GC. This isn't what we want. We want to keep track of that UPROPERTY TObjectPtr 
			// property on the FInstancedStruct payload data, and while it is in the message system queue, keep the references
			// so it doesn't get collected
		}
	};
	
	RunTestTicks(TEXT("AsyncMessage_ReferenceCollection"), TickLambda, /* only tick 10 times to make this go faster */ 10);

	// Always reset test world
	bSuccess &= DestroyTestWorld();

	return bSuccess && !ReportAnyErrors();
}

#endif	// WITH_DEV_AUTOMATION_TESTS
