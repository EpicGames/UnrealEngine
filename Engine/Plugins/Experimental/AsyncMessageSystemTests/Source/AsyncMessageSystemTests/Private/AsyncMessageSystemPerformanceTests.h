// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessageHandle.h"
#include "EngineRuntimeTests.h"			// For AEngineTestTickActor
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/Object.h"				// For UObject

#include "AsyncMessageSystemPerformanceTests.generated.h"

struct FAsyncMessage;
struct FAsyncMessageBindingOptions;
struct FAsyncMessageId;

/**
* A very simple test payload referencing an actor
*/
USTRUCT(NotBlueprintType, meta=(Hidden))
struct FAsyncMessagePerfTestPayload
{
	GENERATED_BODY()
	
	TWeakObjectPtr<AEngineTestTickActor> TargetActor = nullptr;

	// If true, then the test actor will simulate less work
	bool bDoLessWork = false;
};

/**
 * A test actor class which is to be used for async mesage performance testing
 */
UCLASS(NotBlueprintable, NotBlueprintType, meta=(Hidden))
class AASyncMessagePerfTest : public AEngineTestTickActor
{
	GENERATED_BODY()
public:
	
	void SetupBindingToMessage(const FAsyncMessageId& MessageToBindTo, const FAsyncMessageBindingOptions& BindingOpts);	
	void HandleTestCallback(const FAsyncMessage& Message);
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	
	TArray<FAsyncMessageHandle> BoundHandles;

	// A function which simulates doing some CPU floating point work
	void DoTestWork();
	// Properties used in the DoTestWork function above
	float MathCounter = 0.0f;
	float MathIncrement = 0.01f;
	float MathLimit = 1.0f;

	void DoSimpleTestWork();
};


USTRUCT(NotBlueprintType, meta=(Hidden))
struct FTest_RefCollection_Payload
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UObject> ObjPoint = nullptr;
	
	UPROPERTY()
	TWeakObjectPtr<UObject> WeakObjPointer = nullptr;
};

/**
 * A test UObject type which we will use to test ref counting of payload data
 */
UCLASS(NotBlueprintable, NotBlueprintType)
class UTestRefCollectionObject : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY()
	int32 TestValue = 5;
};