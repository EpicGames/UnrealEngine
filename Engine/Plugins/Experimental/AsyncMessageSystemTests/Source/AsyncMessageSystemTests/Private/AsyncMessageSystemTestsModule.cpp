// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageSystemTestsModule.h"
#include "Modules/ModuleManager.h"

bool FAsyncMessageSystemTestsModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded("AsyncMessageSystemTests");
}

IMPLEMENT_MODULE(FAsyncMessageSystemTestsModule, AsyncMessageSystemTests)